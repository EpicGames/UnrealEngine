@echo off

if not exist env (
	..\..\..\Binaries\ThirdParty\Python3\Win64\python.exe -m venv env
)

call .\env\Scripts\activate.bat

python -m pip install --upgrade pip
pip install -r requirements.txt
if %ERRORLEVEL% NEQ 0 (
	rmdir /s /q env
	exit %ERRORLEVEL%
)
