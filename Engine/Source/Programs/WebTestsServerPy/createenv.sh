if [ ! -d "env" ]; then
	case "$OSTYPE" in
		darwin*)
			../../../Binaries/ThirdParty/Python3/Mac/bin/python3 -m venv env
			;;
		*)
			# For daphne4.0.0 -> ImportError: libffi.so.6: cannot open shared object file: No such file or directory
			curl -LO http://archive.ubuntu.com/ubuntu/pool/main/libf/libffi/libffi6_3.2.1-8_amd64.deb
			sudo dpkg -i libffi6_3.2.1-8_amd64.deb
			../../../Binaries/ThirdParty/Python3/Linux/bin/python3 -m venv env
			;;
	esac
fi

. ./env/bin/activate
python -m pip install --upgrade pip
pip install -r requirements.txt
local install_exit_code = $?
if [ $install_exit_code -ne 0 ]; then
	rm -rf env
	exit $install_exit_code
fi
