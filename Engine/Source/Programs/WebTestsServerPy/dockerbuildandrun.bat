@echo off

:: Build image
docker build . -t webtests

set CONTAINER="webtestscontainer"
:: Check if container exist and delete
docker ps -a | findstr /c:"%CONTAINER%" >nul 2>&1
if %errorlevel% equ 0 (
	docker rm -f %CONTAINER%
)

:: Run container with image
docker run --name %CONTAINER% -d -p 8000:8000 webtests
