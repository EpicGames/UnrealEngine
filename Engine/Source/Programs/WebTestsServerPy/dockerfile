FROM python:3.10.4
ENV DockerHOME=/home/app/webtests
RUN mkdir -p $DockerHOME
WORKDIR $DockerHOME
ENV PYTHONDONTWRITEBYTECODE 1
ENV PYTHONUNBUFFERED 1
RUN pip install --upgrade pip
COPY . $DockerHOME
RUN pip install -r requirements.txt
EXPOSE 8000
CMD python manage.py runserver 0.0.0.0:8000
