from django.urls import path

from . import views

urlpatterns = [
    path("httptests/methods", views.methods, name="methods"),
    path("httptests/query_with_params/", views.query_with_params, name="query_with_params"),
    path("httptests/get_data_without_chunks/<int:bytes_number>/<int:repeat_at>/", views.get_data_without_chunks, name="get_data_without_chunks"),
    path("httptests/get_data_gzip/<int:bytes_number>/<int:repeat_at>/", views.get_data_gzip, name="get_data_gzip"),
    path("httptests/streaming_download/<int:chunks>/<int:chunk_size>/<int:chunk_latency>/", views.streaming_download, name="streaming_download"),
    path("httptests/streaming_upload_post", views.streaming_upload_post, name="streaming_upload_post"),
    path("httptests/streaming_upload_put", views.streaming_upload_put, name="streaming_upload_put"),
    path("httptests/redirect_from", views.redirect_from, name="redirect_from"),
    path("httptests/redirect_to", views.redirect_to, name="redirect_to"),
    path("httptests/mock_latency/<int:latency>/", views.mock_latency, name="mock_latency"),
    path("httptests/mock_status/<int:status_code>/", views.mock_status, name="mock_status"),
    path("httptests/post_echo_body", views.post_echo_body, name="post_echo_body"),
]
