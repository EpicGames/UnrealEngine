import asyncio
from datetime import datetime
from django.shortcuts import render, redirect
from django.http.response import JsonResponse
from django.http import JsonResponse
from django.http import HttpResponse
from django.http import StreamingHttpResponse
from django.views.decorators.gzip import gzip_page
from rest_framework.decorators import api_view, parser_classes
from rest_framework.parsers import FileUploadParser
import time
import json
import logging

logger = logging.getLogger('django')

@api_view(['GET', 'POST', 'DELETE', 'PUT', 'PATCH', 'HEAD', 'OPTIONS', 'CONNECT', 'TRACE'])
def methods(request):
    return JsonResponse({})

@api_view(['GET'])
def query_with_params(request):
    var_int = int(request.GET.get('var_int'))
    var_str = request.GET.get('var_str')
    return JsonResponse({'var_int' : var_int, 'var_str' : var_str}, content_type="application/json")

def generate_data(bytes_number, repeat_at):
    assert bytes_number > 0
    assert repeat_at > 0
    assert repeat_at <= 10
    return [i%repeat_at for i in range(bytes_number)]

@api_view(['GET'])
def get_data_without_chunks(request, bytes_number, repeat_at):
    return HttpResponse(generate_data(bytes_number, repeat_at))

@api_view(['GET'])
@gzip_page
def get_data_gzip(request, bytes_number, repeat_at):
    assert request.headers.get('Accept-Encoding') == 'gzip'
    return HttpResponse(generate_data(bytes_number, repeat_at))

@api_view(['GET'])
def streaming_download(request, chunks, chunk_size, chunk_latency):
    async def data_chunk_generator(): # using 'async' for StreamingHttpResponse in asgi
        for x in range(chunks):
            if chunk_latency > 0:
                logger.debug("[%s]sleeping %d seconds", datetime.now().strftime("%H:%M:%S:%f"), chunk_latency)
                await asyncio.sleep(chunk_latency)  # using 'await asyncio.sleep' instead of 'time.sleep' for StreamingHttpResponse in asgi
            data_chunk = 'd' * chunk_size
            logger.debug("[%s]sending %d bytes of data", datetime.now().strftime("%H:%M:%S:%f"), chunk_size)
            yield data_chunk
    response = StreamingHttpResponse(data_chunk_generator())
    response['Content-Length'] = chunks * chunk_size
    return response

def output_file_info(f):
    if f.multiple_chunks():
        # big file
        total_chunks = 0
        total_bytes = 0
        for chunk in f.chunks():
            total_chunks += 1
            total_bytes += len(chunk)
        logger.info("Received %d chunks: %d bytes in total", total_chunks, total_bytes)
    else:
        # small file
        logger.info("Received the whole file: %d bytes", f.size)

@api_view(['POST'])
def streaming_upload_post(request):
    f = request.FILES['file']
    output_file_info(f)
    return JsonResponse({})

@api_view(['PUT'])
@parser_classes((FileUploadParser,))
def streaming_upload_put(request):
    f = request.data['file']
    output_file_info(f)
    return JsonResponse({})

@api_view(['GET'])
def redirect_from(request):
    return redirect('redirect_to')

@api_view(['GET'])
def redirect_to(request):
    return JsonResponse({})

@api_view(['GET'])
def mock_latency(request, latency):
    time.sleep(latency)
    return JsonResponse({})

@api_view(['POST'])
def post_echo_body(request):
    return JsonResponse(request.data)

@api_view(['GET', 'POST', 'DELETE', 'PUT'])
def mock_status(request, status_code):
    json_result = json.dumps({
        "key_a": "value_a",
        "key_b": "value_b"
    })

    headers_to_forward = ['Retry-After',]

    response_headers = {}
    for header_to_forward in headers_to_forward:
        if header_to_forward in request.headers:
            response_headers[header_to_forward] = request.headers[header_to_forward]

    return HttpResponse(json_result, content_type="application/json", status=status_code, headers=response_headers)
