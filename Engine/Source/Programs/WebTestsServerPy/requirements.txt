autobahn==24.4.2
django==4.2.0
djangorestframework==3.14.0
channels==4.0.0
daphne==4.0.0
