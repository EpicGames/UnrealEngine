@echo off

call createenv.bat

python -m daphne -e tcp:port=8000:interface=0.0.0.0 -e tcp:port=8001:interface=0.0.0.0 webtests.asgi:application
