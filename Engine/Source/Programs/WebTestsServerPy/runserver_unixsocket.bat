dotnet build .\unixsockets\unixsocketsserver.csproj -c "Release"
dotnet run --project ".\unixsockets\unixsocketsserver.csproj" -c "Release"