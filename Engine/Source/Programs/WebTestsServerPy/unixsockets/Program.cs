// Copyright Epic Games, Inc. All Rights Reserved.
using Microsoft.AspNetCore.Builder;
using Microsoft.AspNetCore.Hosting;

const string SocketFileName = "webtests.sock";

string CurrentDirectory = Environment.CurrentDirectory;
string SocketsDirectory = Path.Combine(CurrentDirectory, "sockets");

if (Directory.Exists(SocketsDirectory))
{
    //Clean any sockets remaining from a previous run
    foreach(var SocketFile in Directory.EnumerateFiles(SocketsDirectory, "*", SearchOption.AllDirectories))
    {
        File.Delete(SocketFile);
    }
}
else
{
    Directory.CreateDirectory(SocketsDirectory);
}

string SocketPath = Path.Combine(SocketsDirectory, SocketFileName);
Console.WriteLine($"Run Web Tests with the following arguments to test Unix Sockets");
Console.WriteLine($"=> \"Http Methods over Unix Domain Socket\" --extra-args -very_verbose=true -web_server_unix_socket=\"{SocketPath}\"");

var AppBuilder = WebApplication.CreateBuilder();
AppBuilder.WebHost.UseKestrel(Options =>
{
    Options.ListenUnixSocket(SocketPath);
});

var WebServerApp = AppBuilder.Build();

string[] Methods = { "GET", "POST", "PUT", "DELETE" };

WebServerApp.MapMethods("webtests/unixsockettests/{param}", Methods, async Context =>
{
    string Param = (string)Context.Request.RouteValues["param"];
    Context.Response.StatusCode = 200;
    await using (StreamWriter Writer = new StreamWriter(Context.Response.Body, leaveOpen:true))
    {
        await Writer.WriteAsync(Param);
    }
    await Task.CompletedTask;
});

WebServerApp.Run();