import json

from channels.generic.websocket import WebsocketConsumer

class EchoConsumer(WebsocketConsumer):
    def connect(self):
        self.accept()

    def disconnect(self, close_code):
        pass

    def receive(self, text_data):
        self.send(text_data)

class CloseOnReceiveMessageConsumer(WebsocketConsumer):
    def connect(self):
        self.accept()

    def disconnect(self, close_code):
        pass

    def receive(self, text_data):
        self.close()
