from django.urls import re_path

from . import consumers

websocket_urlpatterns = [
    re_path(r"webtests/websocketstests/echo", consumers.EchoConsumer.as_asgi()),
    re_path(r"webtests/websocketstests/close_on_receive_message", consumers.CloseOnReceiveMessageConsumer.as_asgi()),
]
