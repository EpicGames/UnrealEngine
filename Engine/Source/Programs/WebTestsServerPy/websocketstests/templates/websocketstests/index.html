<!DOCTYPE html>
<html>
<head>
    <meta charset="utf-8"/>
    <title>WebSockects Tests</title>
</head>
<body>
    <textarea id="echo-log" cols="100" rows="20"></textarea><br>
    <input id="echo-message-input" type="text" size="100"><br>
    <input id="echo-message-submit" type="button" value="Send">
    <script>
        const echoSocket = new WebSocket(
            'ws://'
            + window.location.host
            + '/webtests/websocketstests/echo/'
        );

        echoSocket.onmessage = function(e) {
            const data = JSON.parse(e.data);
            document.querySelector('#echo-log').value += (data.message + '\n');
        };

        echoSocket.onclose = function(e) {
            console.error('Echo socket closed unexpectedly');
        };

        document.querySelector('#echo-message-input').focus();
        document.querySelector('#echo-message-input').onkeyup = function(e) {
            if (e.key === 'Enter') {  // enter, return
                document.querySelector('#echo-message-submit').click();
            }
        };

        document.querySelector('#echo-message-submit').onclick = function(e) {
            const messageInputDom = document.querySelector('#echo-message-input');
            const message = messageInputDom.value;
            echoSocket.send(JSON.stringify({
                'message': message
            }));
            messageInputDom.value = '';
        };
    </script>
</body>
</html>
