from django.urls import path

from . import views

urlpatterns = [
    path("websocketstests", views.index, name="index"),
]
