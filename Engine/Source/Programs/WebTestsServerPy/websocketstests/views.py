from django.shortcuts import render

def index(request):
    return render(request, "websocketstests/index.html")

